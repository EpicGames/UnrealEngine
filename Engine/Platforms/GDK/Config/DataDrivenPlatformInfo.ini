[DataDrivenPlatformInfo]
bIsConfidential=true
bIsFakePlatform=true
GlobalIdentifier=57103E01479C4A7B993C81B8E1D29907