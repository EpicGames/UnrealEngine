{
	"//0": "The main GDK version values have moved here to support the public PC GDK work",
	"IncludeSDKFile": "$(EngineDir)/Config/Microsoft/GDK_SDK.json",

	"//1": "GDK uses the WindowSDK separately from the GDK version above",
	"MinWindowsSDKVersion": "10.0.19041.0",

	"//2": "all GDK platforms use 'GDK' directory in AutoSDK",
	"AutoSDKPlatform": "GDK",

	"//3": "AutoSDKDirectory can be used to have a patch version in AutoSDK but allowing MainVersion to be the unpatched version number:",
	"//4": "\"AutoSDKDirectory\": \"240302-01\""
}
