[Default]
+SdkInfo=(Category="GDK", Description="GDK April 2026", Install="$(SdkInstallerDir)/GDK/2026-04/2026.04-GDKX--10.0.26100.7822/GDK.exe")
[Options]
+SyncCategory=(UniqueId="a2000523-bbc7-4d2c-88a4-50da4e777ec6", Name="Platform Support: GDK Common", Enable="false", Hidden="true", Paths=".../Platforms/GDK/...")
