// Copyright Epic Games, Inc. All Rights Reserved.

using EpicGames.Core;

namespace UnrealBuildTool
{
	/////////////////////////////////////////////////////////////////////////////////////////
	// If you are for looking version numbers, see Engine/Platforms/GDK/Config/GDK_SDK.json
	/////////////////////////////////////////////////////////////////////////////////////////

	abstract partial class GDKPlatformSDK : MicrosoftPlatformSDK
	{
		/// <summary>
		/// The minimum Windows SDK version to be used. If this is null then it means there is no minimum version
		/// </summary>
		VersionNumber? MinimumWindowsSDKVersion => GetVersionNumberFromConfig("MinWindowsSDKVersion");

		/// <summary>
		/// The maximum Windows SDK version to be used. If this is null then it means "Latest"
		/// </summary>
		VersionNumber? MaximumWindowsSDKVersion => GetVersionNumberFromConfig("MaxWindowsSDKVersion");

		protected override string MainVersionKey => "MainGDKVersion";
		protected override string MinVersionKey => "MinGDKVersion";
		protected override string MaxVersionKey => "MaxGDKVersion";
		protected override string BannedSdkVersionsKey => "BannedGDKVersions";
	}
}
