// Copyright Epic Games, Inc. All Rights Reserved.

using System;
using System.Collections.Generic;
using System.IO;
using System.Runtime.Versioning;
using System.Text.RegularExpressions;
using EpicGames.Core;
using Microsoft.Extensions.Logging;
using UnrealBuildBase;

///////////////////////////////////////////////////////////////////
// If you are looking for supported version numbers, look in the
// GDKPlatformSDK.Versions.cs file next to this file
///////////////////////////////////////////////////////////////////

namespace UnrealBuildTool
{
	abstract partial class GDKPlatformSDK : MicrosoftPlatformSDK
	{
		public GDKPlatformSDK(ILogger InLogger)
			: base(InLogger)
		{
		}

		/// <inheritdoc/>
		protected override string? GetInstalledSDKVersion()
		{
			// get GRDK installed version
			string? InstalledVersion = GDKExports.GetSDKVersion();
			if (InstalledVersion != null)
			{
				// verify the selected GDK edition includes GRDK support
				string GRDKIniFile = Path.Combine(GDKExports.GetGSDKRoot(), InstalledVersion, "GRDK", "grdk.ini");
				FileItem fileItem = FileItem.GetItemByPath(GRDKIniFile); // Use this instead of File.Exists since it caches
				if (fileItem.Exists)
				{
					return InstalledVersion;
				}

				string GRDKHeaderFile = Path.Combine(GDKExports.GetGSDKRoot(), InstalledVersion, "windows", "include", "grdk.h");
				fileItem = FileItem.GetItemByPath(GRDKHeaderFile); // Use this instead of File.Exists since it caches
				if (fileItem.Exists)
				{
					return InstalledVersion;
				}
			}

			// GRDK isn't installed
			return null;
		}

		/// <inheritdoc/>
		public override bool TryConvertVersionToInt(string? StringValue, out ulong OutValue, string? Hint)
		{
			if (StringValue == null)
			{
				OutValue = 0;
				return false;
			}

			// long GDK file naming, typically as Microsoft releases it. YYYY.MM-GDKDetailUpdateQQ-<N.N.N.N>
			Regex LongPattern = new(@"^20(\d\d)[\.\-](\d\d)[\.\-]+((?:[a-zA-Z]*QFE|[a-zA-Z]*Update)(\d+))?");
			Match Result = LongPattern.Match(StringValue);
			if (Result.Success)
			{
				// convert the groups into a string that matches our normal pattern
				StringValue = Result.Groups[1].Value + Result.Groups[2].Value;
				StringValue += String.Format("{0:00}00", Result.Groups[3].Success ? Int32.Parse(Result.Groups[4].Value).ToString("D2") : "00");
				return UInt64.TryParse(StringValue, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
			}

			// 4-part windows-style version number: N.N.N.N, expecting to start with 10.0
			Regex WindowsPattern = new(@"^10.0.(\d*).(\d*)$");
			Result = WindowsPattern.Match(StringValue);
			if (Result.Success)
			{
				// convert it to two hex numbers with 0 padding
				StringValue = String.Format("{0}{1:00000000}", Result.Groups[1].Value, Int32.Parse(Result.Groups[2].Value).ToString("D8"));
				return UInt64.TryParse(StringValue, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
			}

			// YYMMQQ-NN where NN is 01, 02 etc. to signify a specially-selectable custom patched/hacked GDK version, for AutoSDK only.
			Regex PatchedGDKPattern = new(@"^([a-fA-F0-9]{6})\-([a-fA-F0-9]{2})$");
			Result = PatchedGDKPattern.Match(StringValue);
			if (Result.Success)
			{
				StringValue = String.Format("{0:000000}{1:00}", Result.Groups[1].Value, Result.Groups[2].Value);
				if (Result.Groups[1].Value == "00")
				{
					Log.TraceWarningOnce($"Custom GDK version {StringValue} should not have 00 suffix as it cannot be distinguished with the base GDK edition {Result.Groups[0].Value}");
				}
				return UInt64.TryParse(StringValue, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
			}

			//short GDK version format: YYMMQQ
			Regex NormalGDKPattern = new(@"^([a-fA-F0-9]{6})$");
			Result = NormalGDKPattern.Match(StringValue);
			if (Result.Success)
			{
				StringValue = String.Format("{0:000000}00", Result.Groups[1].Value);
				return UInt64.TryParse(StringValue, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
			}

			// unknown - treat version string as if it is just a hex number already
			return UInt64.TryParse(StringValue, System.Globalization.NumberStyles.HexNumber, System.Globalization.CultureInfo.InvariantCulture, out OutValue);
		}

		/// <summary>
		/// The version required to support Visual Studio
		/// </summary>
		public static Version GetVersionRequiredForVisualStudio(VCProjectFileFormat Format)
		{
			return Format >= VCProjectFileFormat.VisualStudio2026
				? new Version(10, 260400)
				: new Version();
		}

		/// <summary>
		/// Whether platform supports switching SDKs during runtime
		/// </summary>
		/// <returns>true if supports</returns>
		protected override bool PlatformSupportsAutoSDKs()
		{
			return true;
		}

		/// <summary>
		/// platforms can choose if they prefer a correct the the AutoSDK install over the manual install.
		/// </summary>
		/// <returns></returns>
		protected override bool PreferAutoSDK()
		{
			return false;
		}

		[SupportedOSPlatform("windows")]
		public static void GetGSDKWindowsSDK(ILogger Logger, UnrealTargetPlatform Platform, out VersionNumber? WindowsSDKVersion, out DirectoryReference? WindowsSDKPath)
		{
			GDKPlatformSDK SDK = (GDKPlatformSDK?)UEBuildPlatformSDK.GetSDKForPlatform(Platform.ToString())!;
			if (!TryGetWindowsSdkDir(null, Logger, out WindowsSDKVersion, out WindowsSDKPath, SDK.MinimumWindowsSDKVersion, SDK.MaximumWindowsSDKVersion))
			{
				throw new BuildException($"Minimum Windows SDK version of {SDK.MinimumWindowsSDKVersion} is required by GDK." + (SDK.MaximumWindowsSDKVersion == null ? "" : $" Maximum version {SDK.MaximumWindowsSDKVersion}"));
			}
		}

		public override string[] GetAllInstalledSDKVersions()
		{
			List<string> GDKVersions = new();

			try
			{
				string GSDKRoot = GDKExports.GetGSDKRoot();
				if (!String.IsNullOrEmpty(GSDKRoot) && Directory.Exists(GSDKRoot))
				{
					string[] SubDirectories = Directory.GetDirectories(GSDKRoot);
					foreach (string SubDirectory in SubDirectories)
					{
						int GDKVersionInt;
						string SDKFilename = Path.Combine(SubDirectory, "GRDK", "grdk.ini");
						FileInfo Info = new(SDKFilename);
						if (Info.Exists && Int32.TryParse(Path.GetFileName(SubDirectory), out GDKVersionInt))
						{
							GDKVersions.Add(GDKVersionInt.ToString());
						}
						else
						{						
							SDKFilename = Path.Combine(SubDirectory, "windows", "include", "grdk.h");
							Info = new(SDKFilename);
							if (Info.Exists && Int32.TryParse(Path.GetFileName(SubDirectory), out GDKVersionInt))
							{
								GDKVersions.Add(GDKVersionInt.ToString());
							}
						}
					}
				}
			}
			catch
			{
			}

			return GDKVersions.ToArray();
		}

		// GDK's .cpp code only relies on GameDK and not GameDKLatest/GameDKCoreLatest - the version is baked in via _GRDK_EDITION
		// this means we can do the SDK switching without having to configure AutoSDK first
		public override bool ManualSDKAutoSwitchRequiresAutoSDK => false;

		protected override bool TryGetEnvironmentForManualSDK(string SelectedSDKVersion, out Dictionary<string, string>? EnvVarValues, out List<string>? PathAdds, out List<string>? PathRemoves)
		{
			EnvVarValues = null;
			PathAdds = null;
			PathRemoves = null;

			string? CurrentSDKVersion = GetInstalledSDKVersion();
			if (CurrentSDKVersion != null && CurrentSDKVersion != SelectedSDKVersion)
			{
				string? GameDK = Environment.GetEnvironmentVariable("GameDK");
				if (GameDK == null)
				{
					return false;
				}

				EnvVarValues = new()
				{
					["GameDKLatest"]     = Path.Combine(GameDK, SelectedSDKVersion), // environment variable used by legacy folder layout
					["GameDKCoreLatest"] = Path.Combine(GameDK, SelectedSDKVersion), // environment variable used by new folder layout (October 2025 onwards)
				};

				return true;
			}

			return false;
		}	
	}
}
