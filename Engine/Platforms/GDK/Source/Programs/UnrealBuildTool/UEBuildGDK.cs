// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool
{
	partial struct UnrealPlatformGroup
	{
		/// <summary>
		/// Add GDK to the known platform groups
		/// </summary>
		public static UnrealPlatformGroup GDK { get; } = FindOrAddByName("GDK");
	}
}

